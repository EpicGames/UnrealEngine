{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Procedural Content Generation Framework (PCG) Water Interop",
	"Description": "Extra plugin for Procedural Content Generation Framework interacting with the Water system.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/latest/en-US/procedural-content-generation--framework-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PCGWaterInterop",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "Water",
			"Enabled": true
		}
	]
}
