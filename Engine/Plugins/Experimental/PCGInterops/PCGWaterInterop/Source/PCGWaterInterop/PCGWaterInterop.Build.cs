// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGWaterInterop : ModuleRules
	{
		public PCGWaterInterop(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"PCG",
					"Water",
				}
			);
		}
	}
}
