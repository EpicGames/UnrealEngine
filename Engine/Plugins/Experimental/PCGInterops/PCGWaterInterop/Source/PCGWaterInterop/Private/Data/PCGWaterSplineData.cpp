// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/PCGWaterSplineData.h"

#include "PCGContext.h"
#include "Data/PCGPointData.h"
#include "Metadata/PCGMetadata.h"

#include "WaterSplineComponent.h"
#include "WaterSplineMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGWaterSplineData)

namespace PCGWaterSplineDataConstants
{
	const FName DepthAttributeName = TEXT("Depth");
	const FName WaterVelocityScalarAttributeName = TEXT("WaterVelocityScalar");
	const FName RiverWidthAttributeName = TEXT("RiverWidth");
	const FName AudioIntensityAttributeName = TEXT("AudioIntensity");
}

void UPCGWaterSplineData::Initialize(const UWaterSplineComponent* InSpline)
{
	check(InSpline && Metadata);

	Super::Initialize(InSpline);

	if (const UWaterSplineMetadata* WaterSplineMetadata = Cast<UWaterSplineMetadata>(InSpline->GetSplinePointsMetadata()))
	{
		WaterSplineMetadataStruct.Depth = WaterSplineMetadata->Depth;
		WaterSplineMetadataStruct.WaterVelocityScalar = WaterSplineMetadata->WaterVelocityScalar;
		WaterSplineMetadataStruct.RiverWidth = WaterSplineMetadata->RiverWidth;
		WaterSplineMetadataStruct.AudioIntensity = WaterSplineMetadata->AudioIntensity;

		Metadata->CreateAttribute<float>(PCGWaterSplineDataConstants::DepthAttributeName, InSpline->WaterSplineDefaults.DefaultDepth, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/true);
		Metadata->CreateAttribute<float>(PCGWaterSplineDataConstants::WaterVelocityScalarAttributeName, InSpline->WaterSplineDefaults.DefaultVelocity, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/true);
		Metadata->CreateAttribute<float>(PCGWaterSplineDataConstants::RiverWidthAttributeName, InSpline->WaterSplineDefaults.DefaultWidth, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/true);
		Metadata->CreateAttribute<float>(PCGWaterSplineDataConstants::AudioIntensityAttributeName, InSpline->WaterSplineDefaults.DefaultAudioIntensity, /*bAllowsInterpolation=*/true, /*bOverrideParent=*/true);
	}
}

bool UPCGWaterSplineData::SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// Find nearest point on spline
	const FVector InPosition = Transform.GetLocation();
	const float NearestPointKey = SplineStruct.FindInputKeyClosestToWorldLocation(InPosition);
	FTransform NearestTransform = SplineStruct.GetTransformAtSplineInputKey(NearestPointKey, ESplineCoordinateSpace::World, true);

	// WaterSpline samples are scaled by river width on X instead of Y, so we swizzle XY to get the correct transform.
	const FVector Scale = NearestTransform.GetScale3D();
	NearestTransform.SetScale3D(FVector(Scale.Y, Scale.X, Scale.Z));

	const FVector LocalPoint = NearestTransform.InverseTransformPosition(InPosition);
	
	// Linear fall off based on the distance to the nearest point
	const float Distance = LocalPoint.Length();
	if (Distance > 1.0f)
	{
		return false;
	}
	else
	{
		OutPoint.Transform = Transform;
		OutPoint.SetLocalBounds(Bounds);
		OutPoint.Density = 1.0f - Distance;

		WriteMetadataToPoint(NearestPointKey, OutPoint, OutMetadata);

		return true;
	}
}

UPCGSpatialData* UPCGWaterSplineData::CopyInternal(FPCGContext* Context) const
{
	UPCGWaterSplineData* NewSplineData = FPCGContext::NewObject_AnyThread<UPCGWaterSplineData>(Context);
	NewSplineData->WaterSplineMetadataStruct = WaterSplineMetadataStruct;

	CopySplineData(NewSplineData);

	return NewSplineData;
}

FTransform UPCGWaterSplineData::GetTransformAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace, FBox* OutBounds) const
{
	FTransform Transform = Super::GetTransformAtDistance(SegmentIndex, Distance, bWorldSpace, OutBounds);

	// WaterSpline samples are scaled by river width on X instead of Y, so we swizzle XY to get the correct transform.
	const FVector Scale = Transform.GetScale3D();
	Transform.SetScale3D(FVector(Scale.Y, Scale.X, Scale.Z));

	return Transform;
}

void UPCGWaterSplineData::WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const
{
	// TODO: Might be interesting for those 4 custom attributes to follow the normal flow for attribute interpolation
	// instead of having a custom interpolation here.
	Super::WriteMetadataToPoint(InputKey, OutPoint, OutMetadata);

	if (!OutMetadata)
	{
		return;
	}

	OutMetadata->InitializeOnSet(OutPoint.MetadataEntry);

	if (FPCGMetadataAttribute<float>* DepthAttribute = OutMetadata->GetMutableTypedAttribute<float>(PCGWaterSplineDataConstants::DepthAttributeName))
	{
		DepthAttribute->SetValue(OutPoint.MetadataEntry, WaterSplineMetadataStruct.Depth.Eval(InputKey));
	}

	if (FPCGMetadataAttribute<float>* VelocityAttribute = OutMetadata->GetMutableTypedAttribute<float>(PCGWaterSplineDataConstants::WaterVelocityScalarAttributeName))
	{
		VelocityAttribute->SetValue(OutPoint.MetadataEntry, WaterSplineMetadataStruct.WaterVelocityScalar.Eval(InputKey));
	}

	if (FPCGMetadataAttribute<float>* WidthAttribute = OutMetadata->GetMutableTypedAttribute<float>(PCGWaterSplineDataConstants::RiverWidthAttributeName))
	{
		WidthAttribute->SetValue(OutPoint.MetadataEntry, WaterSplineMetadataStruct.RiverWidth.Eval(InputKey));
	}

	if (FPCGMetadataAttribute<float>* AudioIntensityAttribute = OutMetadata->GetMutableTypedAttribute<float>(PCGWaterSplineDataConstants::AudioIntensityAttributeName))
	{
		AudioIntensityAttribute->SetValue(OutPoint.MetadataEntry, WaterSplineMetadataStruct.AudioIntensity.Eval(InputKey));
	}
}
