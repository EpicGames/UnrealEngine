// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetWaterSpline.h"

#include "Data/PCGWaterSplineData.h"

#include "WaterSplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetWaterSpline)

#define LOCTEXT_NAMESPACE "PCGGetWaterSplineElement"

UPCGGetWaterSplineSettings::UPCGGetWaterSplineSettings()
{
	Mode = EPCGGetDataFromActorMode::ParseActorComponents;
}

#if WITH_EDITOR
FText UPCGGetWaterSplineSettings::GetNodeTooltipText() const
{
	return LOCTEXT("GetWaterSplineTooltip", "Builds a collection of data from WaterSplineComponents on the selected actors.");
}
#endif

FPCGElementPtr UPCGGetWaterSplineSettings::CreateElement() const
{
	return MakeShared<FPCGGetWaterSplineElement>();
}

void FPCGGetWaterSplineElement::ProcessActor(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, AActor* FoundActor) const
{
	check(Context && Settings);

	if (!FoundActor || !IsValid(FoundActor))
	{
		return;
	}

	auto NameTagsToStringTags = [](const FName& InName) { return InName.ToString(); };
	TSet<FString> ActorTags;
	Algo::Transform(FoundActor->Tags, ActorTags, NameTagsToStringTags);

	// Prepare data on a component basis
	TInlineComponentArray<UWaterSplineComponent*, 4> WaterSplines;
	FoundActor->GetComponents(WaterSplines);

	for (UWaterSplineComponent* WaterSplineComponent : WaterSplines)
	{
		UPCGWaterSplineData* SplineData = NewObject<UPCGWaterSplineData>();
		SplineData->Initialize(WaterSplineComponent);

		FPCGTaggedData& TaggedData = Context->OutputData.TaggedData.Emplace_GetRef();
		TaggedData.Data = SplineData;
		Algo::Transform(WaterSplineComponent->ComponentTags, TaggedData.Tags, NameTagsToStringTags);
		TaggedData.Tags.Append(ActorTags);
	}
}

#undef LOCTEXT_NAMESPACE
