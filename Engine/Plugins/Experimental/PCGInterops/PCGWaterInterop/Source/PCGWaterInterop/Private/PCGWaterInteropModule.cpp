// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGWaterInteropModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FPCGWaterInteropModule, PCGWaterInterop);

DEFINE_LOG_CATEGORY(LogPCGWaterInterop);
