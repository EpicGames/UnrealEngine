// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/PCGSplineData.h"

#include "PCGWaterSplineData.generated.h"

class UWaterSplineComponent;

/** Helper struct to hold a copy of the WaterSplineMetadata contents. */
USTRUCT()
struct FPCGWaterSplineMetadataStruct
{
	GENERATED_BODY()

	/** The Depth of the water at this vertex. */
	UPROPERTY()
	FInterpCurveFloat Depth;

	/** The Current of the water at this vertex. Magnitude and direction. */
	UPROPERTY()
	FInterpCurveFloat WaterVelocityScalar;

	/** Rivers Only: The width of the river (from center) in each direction. */
	UPROPERTY()
	FInterpCurveFloat RiverWidth;

	/** A scalar used to define intensity of the water audio along the spline. */
	UPROPERTY()
	FInterpCurveFloat AudioIntensity;
};

UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGWaterSplineData : public UPCGSplineData
{
	GENERATED_BODY()

public:
	void Initialize(const UWaterSplineComponent* InSpline);

	//~Begin UPCGSpatialData interface
	virtual bool SamplePoint(const FTransform& Transform, const FBox& Bounds, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
protected:
	virtual UPCGSpatialData* CopyInternal(FPCGContext* Context) const override;
	//~End UPCGSpatialData interface

	//~Begin UPCGPolyLineData interface
	virtual FTransform GetTransformAtDistance(int SegmentIndex, FVector::FReal Distance, bool bWorldSpace = true, FBox* OutBounds = nullptr) const override;
	virtual void WriteMetadataToPoint(float InputKey, FPCGPoint& OutPoint, UPCGMetadata* OutMetadata) const override;
	//~End UPCGPolyLineData interface

	UPROPERTY()
	FPCGWaterSplineMetadataStruct WaterSplineMetadataStruct;
};
