// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"
#include "Elements/PCGDataFromActor.h"
#include "Elements/PCGTypedGetter.h"

#include "PCGGetWaterSpline.generated.h"

/** Builds a collection of data from WaterSplineComponents on the selected actors. */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGGetWaterSplineSettings : public UPCGGetSplineSettings
{
	GENERATED_BODY()

public:
	UPCGGetWaterSplineSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("GetWaterSplineData")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGGetWaterSplineElement", "NodeTitle", "Get Water Spline Data"); }
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings
};

class FPCGGetWaterSplineElement : public FPCGDataFromActorElement
{
protected:
	virtual void ProcessActor(FPCGContext* Context, const UPCGDataFromActorSettings* Settings, AActor* FoundActor) const override;
};
